// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

[SupportedTargetTypes(TargetType.Editor, TargetType.Program)]
public class ConfigEditor : ModuleRules
{
	public ConfigEditor(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicIncludePaths.AddRange(
			new string[] {
				"Editor/ConfigEditor/Public/PropertyVisualization",
			}
		);

		PublicIncludePathModuleNames.AddRange(
			new string[] {
				"PropertyEditor",
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"PropertyEditor",
				"Slate",
				"SlateCore",
				"SourceControl",
				"TargetPlatform",
			}
		);


		CircularlyReferencedDependentModules.AddRange(
			new string[] 
			{
				"PropertyEditor",
			}
		); 


		PrivateIncludePathModuleNames.AddRange(
			new string[] {
			}
		);

		DynamicallyLoadedModuleNames.AddRange(
			new string[] {
			}
		);
	}
}
