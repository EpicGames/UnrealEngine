// Copyright Epic Games, Inc. All Rights Reserved.


#include "ConfigEditorModule.h"

#include "Delegates/Delegate.h"
#include "Framework/Docking/TabManager.h"
#include "HAL/Platform.h"
#include "Internationalization/Internationalization.h"
#include "Logging/LogMacros.h"
#include "Modules/ModuleManager.h"
#include "SConfigEditor.h"
#include "Styling/AppStyle.h"
#include "Textures/SlateIcon.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Docking/SDockTab.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

DEFINE_LOG_CATEGORY_STATIC(ConfigEditor, Log, All);

/*-----------------------------------------------------------------------------
   FConfigEditorModule
-----------------------------------------------------------------------------*/

namespace ConfigEditorModule
{
	static const FName ConfigEditorId = FName(TEXT("ConfigEditor"));
}


void FConfigEditorModule::StartupModule()
{
	FGlobalTabmanager::Get()->RegisterNomadTabSpawner(ConfigEditorModule::ConfigEditorId, FOnSpawnTab::CreateRaw(this, &FConfigEditorModule::SpawnConfigEditorTab))
		.SetDisplayName(NSLOCTEXT("ConfigEditorModule", "TabTitle", "Config Editor"))
		.SetTooltipText(NSLOCTEXT("ConfigEditorModule", "TooltipText", "Open the Config Editor tab."))
		.SetIcon(FSlateIcon(FAppStyle::GetAppStyleSetName(), "ConfigEditor.TabIcon"));
}


void FConfigEditorModule::ShutdownModule()
{

}


void FConfigEditorModule::AddExternalPropertyValueWidgetAndConfigPairing(const FString& ConfigFile, const TSharedPtr<SWidget> ValueWidget)
{
	ExternalPropertyValueWidgetAndConfigPairings.Add(ConfigFile, ValueWidget);
}


TSharedRef<SWidget> FConfigEditorModule::GetValueWidgetForConfigProperty(const FString& ConfigFile)
{
	TSharedPtr<SWidget>* ValueWidget = ExternalPropertyValueWidgetAndConfigPairings.Find(ConfigFile);
	return (ValueWidget != nullptr && (*ValueWidget).IsValid()) ? (*ValueWidget).ToSharedRef() : SNullWidget::NullWidget;
}


TSharedRef<SDockTab> FConfigEditorModule::SpawnConfigEditorTab(const FSpawnTabArgs& Args)
{
	return SNew(SDockTab)
		.TabRole(ETabRole::NomadTab)
		[
			SAssignNew(PropertyConfigEditor, SConfigEditor, CachedPropertyToView)
		];
}


void FConfigEditorModule::CreateHierarchyEditor(FProperty* InEditProperty)
{
	CachedPropertyToView = InEditProperty;
}


IMPLEMENT_MODULE(FConfigEditorModule, ConfigEditor);
