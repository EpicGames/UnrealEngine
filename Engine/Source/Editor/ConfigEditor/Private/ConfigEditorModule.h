// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

// Module includes

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "IConfigEditorModule.h"
#include "Templates/SharedPointer.h"
#include "UObject/WeakFieldPtr.h"

class FProperty;
class FSpawnTabArgs;
class SConfigEditor;
class SWidget;

/*-----------------------------------------------------------------------------
   FConfigEditorModule
-----------------------------------------------------------------------------*/
class FConfigEditorModule : public IConfigEditorModule
{
public:
	//~ Begin IModuleInterface Interface
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	//~ End IModuleInterface Interface


public:
	//~ Begin IConfigEditorModule Interface
	virtual void CreateHierarchyEditor(FProperty* EditProperty) override;
	virtual void AddExternalPropertyValueWidgetAndConfigPairing(const FString& ConfigFile, const TSharedPtr<SWidget> ValueWidget) override;
	virtual TSharedRef<SWidget> GetValueWidgetForConfigProperty(const FString& ConfigFile) override;
	//~ End IConfigEditorModule Interface	
	

private:
	/** 
	 * Creates a new Config editor tab 
	 *
	 * @return A spawned tab containing a config editor for a pre-defined property
	 */
	TSharedRef<class SDockTab> SpawnConfigEditorTab(const FSpawnTabArgs& Args);


private:
	// We use this to maintain a reference to a property value widget, ie. a combobox for enum, a check box for bool...
	// and provide these references when they are needed later in the details view construction
	// This is important.
	TMap<FString, TSharedPtr<SWidget>> ExternalPropertyValueWidgetAndConfigPairings;

	// Reference to the Config editor widget
	TSharedPtr<SConfigEditor> PropertyConfigEditor;

	// Reference to the property the hierarchy is to view.
	TWeakFieldPtr<FProperty> CachedPropertyToView;
};

