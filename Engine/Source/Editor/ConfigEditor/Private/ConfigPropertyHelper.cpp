// Copyright Epic Games, Inc. All Rights Reserved.

#include "ConfigPropertyHelper.h"

#include "HAL/FileManager.h"
#include "ISourceControlModule.h"
#include "ISourceControlState.h"
#include "Misc/Paths.h"
#include "Templates/SharedPointer.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ConfigPropertyHelper)

class ISourceControlProvider;

void UPropertyConfigFileDisplayRow::InitWithConfigAndProperty(const FString& InConfigFileName, FProperty* InEditProperty)
{
	// cache both relative (../../../Proj/Config/Foo.ini) and absolute paths (D:\UE\Proj\Config\Foo.ini) 
	NormalizedConfigFileName = InConfigFileName;
	ConfigFileName = FPaths::ConvertRelativePathToFull(InConfigFileName);
	ExternalProperty = InEditProperty;


	ISourceControlProvider& SourceControlProvider = ISourceControlModule::Get().GetProvider();
	// We will add source control soon...
	FSourceControlStatePtr SourceControlState = nullptr; // SourceControlProvider.GetState(ConfigFileName, EStateCacheUsage::Use);

	// Only include config files that are currently checked out or packages not under source control
	{
		if (FPaths::FileExists(ConfigFileName))
		{
			if (SourceControlState.IsValid())
			{
				bIsFileWritable = SourceControlState->IsCheckedOut() || SourceControlState->IsAdded();
			}
			else
			{
				bIsFileWritable = !IFileManager::Get().IsReadOnly(*ConfigFileName);
			}
		}
		else
		{
			if (SourceControlState.IsValid())
			{
				bIsFileWritable = (SourceControlState->IsSourceControlled() && SourceControlState->CanAdd());
			}
			else
			{
				// if the file is not in source control, it is "writeable" ie createable
				bIsFileWritable = true;
			}
		}
	}

}
