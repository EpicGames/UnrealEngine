// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyVisualization/ConfigPropertyCellPresenter.h"

#include "ConfigPropertyHelper.h"
#include "Containers/Array.h"
#include "CoreGlobals.h"
#include "Delegates/Delegate.h"
#include "HAL/Platform.h"
#include "IConfigEditorModule.h"
#include "Input/Reply.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Layout/Visibility.h"
#include "Misc/Attribute.h"
#include "Misc/ConfigCacheIni.h"
#include "Modules/ModuleManager.h"
#include "PropertyHandle.h"
#include "SlotBase.h"
#include "Styling/AppStyle.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Styling/SlateColor.h"
#include "Templates/Casts.h"
#include "Types/SlateEnums.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/FieldPath.h"
#include "UObject/NameTypes.h"
#include "UObject/UnrealType.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/Images/SImage.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/SWidget.h"

class UObject;
struct FGeometry;
struct FPointerEvent;


#define LOCTEXT_NAMESPACE "ConfigEditor"


class SConfigPropertyCell
	: public SCompoundWidget
{
public:
	SLATE_USER_ARGS(SConfigPropertyCell) {}
	SLATE_END_ARGS()

	// Begin SCompoundWidget|SWidget interface
	virtual void Construct(const FArguments& InArgs, UPropertyConfigFileDisplayRow* InPropertyOwnerDisplayObject);
	virtual void OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent) override;
	virtual void OnMouseLeave(const FPointerEvent& MouseEvent) override;
	// End SCompoundWidget|SWidget interface

private:

	// Create the displayable area for this cell.
	void BuildDisplayAreaWidget();
	// The shown display widget, includes all optional decoration
	TSharedPtr<SWidget> DecoratedDisplayWidget;

	// The control widget for editing the value of the property in the specified config.
	TSharedPtr<SWidget> DisplayedValueWidget;
	EVisibility GetDisplayedValueWidgetVisibility() const;
	bool HandleDisplayedValueWidgetEnabled() const;

	// The capacity to add an entry to the specified config file
	TSharedPtr<SButton> AddPropertyToConfigButton;
	FReply HandleAddPropertyToConfigClicked();
	EVisibility GetAddPropertyToConfigButtonVisibility() const;

	// WIP.. The capacity to remove an entry from the specified config file
	TSharedPtr<SButton> RemovePropertyFromConfigButton;
	FReply HandleRemovePropertyFromConfigClicked();
	EVisibility GetRemovePropertyFromConfigVisibility() const;

	// Check if the config file has an entry for this property
	bool DoesConfigFileHaveEntryForProperty() const;
	// Cached value of the above, as we use this for widget visibility callbacks.
	bool bCachedConfigHasPropertyValue;

	// The object which holds the property we are editing
	UPropertyConfigFileDisplayRow* PropertyOwnerDisplayObject;

	FConfigCacheIni* ConfigSystem;
};


void SConfigPropertyCell::BuildDisplayAreaWidget()
{
	IConfigEditorModule& ConfigEditorModule = FModuleManager::Get().GetModuleChecked<IConfigEditorModule>("ConfigEditor");
	DisplayedValueWidget = ConfigEditorModule.GetValueWidgetForConfigProperty(PropertyOwnerDisplayObject->ConfigFileName);
	DisplayedValueWidget->SetEnabled(TAttribute<bool>::Create(TAttribute<bool>::FGetter::CreateSP(this, &SConfigPropertyCell::HandleDisplayedValueWidgetEnabled)));

	static const FName DefaultForegroundName("DefaultForeground");

	SAssignNew(DecoratedDisplayWidget, SHorizontalBox)
		// The widget to alter the objects value in a config file
		+ SHorizontalBox::Slot()
		[
			SNew(SVerticalBox)
			.Visibility(this, &SConfigPropertyCell::GetDisplayedValueWidgetVisibility)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SNew(SHorizontalBox)
				+ SHorizontalBox::Slot()
				.HAlign(HAlign_Center)
				[
					SNew(SBorder)
					.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
					.ForegroundColor(FAppStyle::GetSlateColor(DefaultForegroundName))
					.Padding(0.0f)
					[
						DisplayedValueWidget.ToSharedRef()
					]
				]
			]
		]
		// The add to config widget
		+ SHorizontalBox::Slot()
		.HAlign(HAlign_Center)
		[
			SNew(SVerticalBox)
			.Visibility(this, &SConfigPropertyCell::GetAddPropertyToConfigButtonVisibility)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(EVerticalAlignment::VAlign_Center)
			[
				SAssignNew(AddPropertyToConfigButton, SButton)
				.OnClicked(this, &SConfigPropertyCell::HandleAddPropertyToConfigClicked)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Plus")))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		]
		// The remove from config widget
		+ SHorizontalBox::Slot()
		.AutoWidth()
		.HAlign(HAlign_Center)
		[
			SNew(SVerticalBox)
			.Visibility(this, &SConfigPropertyCell::GetRemovePropertyFromConfigVisibility)
			+ SVerticalBox::Slot()
			.AutoHeight()
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(FMargin(4.0f, 0.0f))
			[
				SAssignNew(RemovePropertyFromConfigButton, SButton)
				.OnClicked(this, &SConfigPropertyCell::HandleRemovePropertyFromConfigClicked)
				.ButtonStyle(FAppStyle::Get(), "HoverHintOnly")
				[
					SNew(SImage)
					.Image(FAppStyle::GetBrush(TEXT("Cross")))
					.ColorAndOpacity(FSlateColor::UseForeground())
				]
			]
		];
}


void SConfigPropertyCell::Construct(const FArguments& InArgs, UPropertyConfigFileDisplayRow* InPropertyOwnerDisplayObject)
{
	PropertyOwnerDisplayObject = InPropertyOwnerDisplayObject;
	bCachedConfigHasPropertyValue = DoesConfigFileHaveEntryForProperty();

	// Create the cell visualization
	BuildDisplayAreaWidget();

	ChildSlot
	[
		DecoratedDisplayWidget.ToSharedRef()
	];
}


void SConfigPropertyCell::OnMouseEnter(const FGeometry& MyGeometry, const FPointerEvent& MouseEvent)
{

}


void SConfigPropertyCell::OnMouseLeave(const FPointerEvent& MouseEvent)
{

}


EVisibility SConfigPropertyCell::GetDisplayedValueWidgetVisibility() const
{
	return bCachedConfigHasPropertyValue ? EVisibility::Visible : EVisibility::Collapsed;
}


bool SConfigPropertyCell::HandleDisplayedValueWidgetEnabled() const
{
	return PropertyOwnerDisplayObject->bIsFileWritable;
}


FReply SConfigPropertyCell::HandleAddPropertyToConfigClicked()
{
	bCachedConfigHasPropertyValue = true;

	// force a "OnValueChanged" in the customization here by triggering a PropertyChanged event
	FPropertyChangedEvent GeneratedOutputChangedEvent(PropertyOwnerDisplayObject->ExternalProperty.Get(), EPropertyChangeType::ValueSet);
	FCoreUObjectDelegates::OnObjectPropertyChanged.Broadcast(PropertyOwnerDisplayObject->TempConfigObject.Get(), GeneratedOutputChangedEvent);

	return FReply::Handled();
}


EVisibility SConfigPropertyCell::GetAddPropertyToConfigButtonVisibility() const
{
	return (!bCachedConfigHasPropertyValue && PropertyOwnerDisplayObject->bIsFileWritable) ? EVisibility::Visible : EVisibility::Collapsed;
}


FReply SConfigPropertyCell::HandleRemovePropertyFromConfigClicked()
{
	// remove the property from the layer
	UClass* OwnerClass = PropertyOwnerDisplayObject->ExternalProperty->GetOwnerClass();
	FString SectionName = OwnerClass->GetPathName();
	FString PropertyName = PropertyOwnerDisplayObject->ExternalProperty->GetName();
	GConfig->ResetKeyInSectionOfStaticLayer(*SectionName, *PropertyName, OwnerClass->GetConfigName(), PropertyOwnerDisplayObject->NormalizedConfigFileName, true);

	// Update the CDO, as this change might have had an impact on it's value.
	OwnerClass->GetDefaultObject()->ReloadConfig();

	bCachedConfigHasPropertyValue = false;

	return FReply::Handled();
}


EVisibility SConfigPropertyCell::GetRemovePropertyFromConfigVisibility() const
{
	return DoesConfigFileHaveEntryForProperty() ? EVisibility::Visible : EVisibility::Collapsed;
}


bool SConfigPropertyCell::DoesConfigFileHaveEntryForProperty() const
{
	UClass* OwnerClass = PropertyOwnerDisplayObject->ExternalProperty->GetOwnerClass();
	FString SectionName = OwnerClass->GetPathName();
	FName PropertyName = PropertyOwnerDisplayObject->ExternalProperty->GetFName();

	if (const FConfigBranch* Branch = GConfig->FindBranch(*OwnerClass->GetConfigName(), OwnerClass->GetConfigName()))
	{
		if (const FConfigCommandStream* Layer = Branch->GetStaticLayer(PropertyOwnerDisplayObject->NormalizedConfigFileName))
		{
			if (const FConfigCommandStreamSection* Sec = Layer->Find(SectionName))
			{
				return Sec->Contains(PropertyName);
			}
		}
	}
	return false;
}



FConfigPropertyCellPresenter::FConfigPropertyCellPresenter(const TSharedPtr< IPropertyHandle > PropertyHandle)
{
	TArray<UObject*> OuterObjects;
	PropertyHandle->GetOuterObjects(OuterObjects);

	if (OuterObjects.Num() == 1)
	{
		UPropertyConfigFileDisplayRow* PropertyOwner = CastChecked<UPropertyConfigFileDisplayRow>(OuterObjects[0]);
		DisplayWidget = SNew(SConfigPropertyCell, PropertyOwner);
	}
}


TSharedRef< class SWidget > FConfigPropertyCellPresenter::ConstructDisplayWidget()
{
	return DisplayWidget.ToSharedRef();
}


bool FConfigPropertyCellPresenter::RequiresDropDown()
{
	return false;
}


TSharedRef< class SWidget > FConfigPropertyCellPresenter::ConstructEditModeCellWidget()
{
	return ConstructDisplayWidget();
}


TSharedRef< class SWidget > FConfigPropertyCellPresenter::ConstructEditModeDropDownWidget()
{
	return SNullWidget::NullWidget;
}


TSharedRef< class SWidget > FConfigPropertyCellPresenter::WidgetToFocusOnEdit()
{
	return SNullWidget::NullWidget;
}


bool FConfigPropertyCellPresenter::HasReadOnlyEditMode()
{
	return true;
}


FString FConfigPropertyCellPresenter::GetValueAsString()
{
	return FString();
}


FText FConfigPropertyCellPresenter::GetValueAsText()
{
	return FText();
}


#undef LOCTEXT_NAMESPACE
