// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Internationalization/Text.h"
#include "IPropertyTableCellPresenter.h"
#include "Templates/SharedPointer.h"

class IPropertyHandle;
class SWidget;

class FConfigPropertyCellPresenter : public TSharedFromThis< FConfigPropertyCellPresenter >, public IPropertyTableCellPresenter
{
public:

	FConfigPropertyCellPresenter(const TSharedPtr< IPropertyHandle > PropertyHandle);
	virtual ~FConfigPropertyCellPresenter() = default;


	/** Begin IPropertyTableCellPresenter interface */
	virtual TSharedRef< class SWidget > ConstructDisplayWidget() override;
	virtual bool RequiresDropDown() override;
	virtual TSharedRef< class SWidget > ConstructEditModeCellWidget() override;
	virtual TSharedRef< class SWidget > ConstructEditModeDropDownWidget() override;
	virtual TSharedRef< class SWidget > WidgetToFocusOnEdit() override;
	virtual bool HasReadOnlyEditMode() override;
	virtual FString GetValueAsString() override;
	virtual FText GetValueAsText() override;
	/** End IPropertyTableCellPresenter interface */

protected:
	// The cell content
	TSharedPtr<SWidget> DisplayWidget;
};
