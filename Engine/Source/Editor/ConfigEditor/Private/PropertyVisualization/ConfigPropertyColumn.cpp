// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyVisualization/ConfigPropertyColumn.h"

#include "HAL/Platform.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableColumn.h"
#include "Internationalization/Text.h"
#include "Misc/Attribute.h"
#include "PropertyPath.h"
#include "PropertyVisualization/ConfigPropertyCellPresenter.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

class IPropertyHandle;
class IPropertyTableUtilities;
class SWidget;



#define LOCTEXT_NAMESPACE "ConfigEditor"


bool FConfigPropertyCustomColumn::Supports(const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities) const
{
	bool IsSupported = false;

	if (Column->GetDataSource()->IsValid())
	{
		TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
		if (PropertyPath.IsValid() && PropertyPath->GetNumProperties() > 0)
		{
			const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
			const FProperty* Property = PropertyInfo.Property.Get();
			IsSupported = Property->GetFName() == TEXT("ExternalProperty");
		}
	}

	return IsSupported;
}

TSharedPtr< SWidget > FConfigPropertyCustomColumn::CreateColumnLabel(const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style) const
{
	if (Column->GetDataSource()->IsValid())
	{
		TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
		if (PropertyPath.IsValid() && PropertyPath->GetNumProperties() > 0)
		{
			return SNew(STextBlock)
				.Text(EditProperty->GetDisplayNameText());
		}
	}
	return SNullWidget::NullWidget;
}


TSharedPtr< IPropertyTableCellPresenter > FConfigPropertyCustomColumn::CreateCellPresenter(const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style) const
{
	TSharedPtr< IPropertyHandle > PropertyHandle = Cell->GetPropertyHandle();
	if (PropertyHandle.IsValid())
	{
		return MakeShareable(new FConfigPropertyCellPresenter(PropertyHandle));
	}

	return nullptr;
}


#undef LOCTEXT_NAMESPACE
