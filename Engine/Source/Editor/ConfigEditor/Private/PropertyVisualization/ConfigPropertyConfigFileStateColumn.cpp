// Copyright Epic Games, Inc. All Rights Reserved.

#include "PropertyVisualization/ConfigPropertyConfigFileStateColumn.h"

#include "HAL/PlatformCrt.h"
#include "IPropertyTable.h"
#include "IPropertyTableCell.h"
#include "IPropertyTableColumn.h"
#include "Internationalization/Internationalization.h"
#include "PropertyHandle.h"
#include "PropertyPath.h"
#include "SlotBase.h"
#include "UObject/UnrealType.h"
#include "UObject/WeakFieldPtr.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/SNullWidget.h"
#include "Widgets/Text/STextBlock.h"

class IPropertyTableUtilities;
class SWidget;



#define LOCTEXT_NAMESPACE "ConfigEditor"


FConfigPropertyConfigFileStateCellPresenter::FConfigPropertyConfigFileStateCellPresenter(const TSharedPtr< IPropertyHandle > PropertyHandle)
{
	//		Text = GetPropertyAsText(PropertyHandle);
}


TSharedRef< class SWidget > FConfigPropertyConfigFileStateCellPresenter::ConstructDisplayWidget()
{
	return
		SNew(SHorizontalBox)
		+ SHorizontalBox::Slot()
		.AutoWidth()
		[
			SNew(STextBlock)
			.Text(LOCTEXT("SCC unconnected","SCC"))
		];
}


bool FConfigPropertyConfigFileStateCellPresenter::RequiresDropDown()
{
	return false;
}


TSharedRef< class SWidget > FConfigPropertyConfigFileStateCellPresenter::ConstructEditModeCellWidget()
{
	return ConstructDisplayWidget();
}


TSharedRef< class SWidget > FConfigPropertyConfigFileStateCellPresenter::ConstructEditModeDropDownWidget()
{
	return SNullWidget::NullWidget;
}


TSharedRef< class SWidget > FConfigPropertyConfigFileStateCellPresenter::WidgetToFocusOnEdit()
{
	return SNullWidget::NullWidget;
}


bool FConfigPropertyConfigFileStateCellPresenter::HasReadOnlyEditMode()
{
	return true;
}


FString FConfigPropertyConfigFileStateCellPresenter::GetValueAsString()
{
	return Text.ToString();
}


FText FConfigPropertyConfigFileStateCellPresenter::GetValueAsText()
{
	return Text;
}


bool FConfigPropertyConfigFileStateCustomColumn::Supports(const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities) const
{
	bool IsSupported = false;

	if (Column->GetDataSource()->IsValid())
	{
		TSharedPtr< FPropertyPath > PropertyPath = Column->GetDataSource()->AsPropertyPath();
		if (PropertyPath.IsValid() && PropertyPath->GetNumProperties() > 0)
		{
			const FPropertyInfo& PropertyInfo = PropertyPath->GetRootProperty();
			const FProperty* Property = PropertyInfo.Property.Get();
			if (SupportedProperty == Property)
			{
				IsSupported = true;
			}
		}
	}

	return IsSupported;
}

TSharedPtr< SWidget > FConfigPropertyConfigFileStateCustomColumn::CreateColumnLabel(const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style) const
{
	return NULL;
}

TSharedPtr< IPropertyTableCellPresenter > FConfigPropertyConfigFileStateCustomColumn::CreateCellPresenter(const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style) const
{
	TSharedPtr< IPropertyHandle > PropertyHandle = Cell->GetPropertyHandle();
	if (PropertyHandle.IsValid())
	{
		TSharedRef<FPropertyPath> Path =  FPropertyPath::Create(PropertyHandle->GetProperty());
		return MakeShareable(new FConfigPropertyConfigFileStateCellPresenter(PropertyHandle));
	}

	return NULL;
	/*
	SNew(SPropertyTableCell, Cell)
		.Presenter(CellPresenter)
		.Style(Style);
	*/
}


#undef LOCTEXT_NAMESPACE
