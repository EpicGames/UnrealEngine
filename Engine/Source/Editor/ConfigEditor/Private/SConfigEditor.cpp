// Copyright Epic Games, Inc. All Rights Reserved.

#include "SConfigEditor.h"

#include "ConfigPropertyHelper.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "DetailsViewArgs.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "IDetailsView.h"
#include "Layout/Children.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/ConfigTypes.h"
#include "Modules/ModuleManager.h"
#include "PropertyEditorModule.h"
#include "STargetPlatformSelector.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Templates/Tuple.h"
#include "UObject/Class.h"
#include "UObject/FieldPath.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"
#include "Widgets/SBoxPanel.h"


#define LOCTEXT_NAMESPACE "ConfigEditor"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void SConfigEditor::Construct(const FArguments& InArgs, TWeakFieldPtr<FProperty> InEditProperty)
{
	TargetPlatformSelection = SNew(STargetPlatformSelector)
		.OnTargetPlatformChanged(this, &SConfigEditor::HandleTargetPlatformChanged);

	EditProperty = InEditProperty;

	// initialize details view
	FDetailsViewArgs DetailsViewArgs;
	{
		DetailsViewArgs.bAllowSearch = false;
		DetailsViewArgs.bHideSelectionTip = true;
		DetailsViewArgs.bLockable = false;
		DetailsViewArgs.bSearchInitialKeyFocus = true;
		DetailsViewArgs.bUpdatesFromSelection = false;
		DetailsViewArgs.bShowOptions = false;
		DetailsViewArgs.bShowModifiedPropertiesOption = false;
	}

	DetailsView = FModuleManager::GetModuleChecked<FPropertyEditorModule>("PropertyEditor").CreateDetailView(DetailsViewArgs);
	
	CreateDisplayObjectForSelectedTargetPlatform();

	////
	// Our Widget setup is complete.
	PropertyValueEditor = DetailsView.ToSharedRef();

	ChildSlot
	[
		SNew(SVerticalBox)
		+ SVerticalBox::Slot()
		.AutoHeight()
		[
			TargetPlatformSelection.ToSharedRef()
		]
		+ SVerticalBox::Slot()
		[
			PropertyValueEditor.ToSharedRef()
		]
	];
}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION

void SConfigEditor::CreateDisplayObjectForSelectedTargetPlatform()
{
	// Get the ini file and hierarchy linked with this property
	FString SelectedTargetPlatform = *TargetPlatformSelection->GetSelectedTargetPlatform().Get();

	FString ConfigHelperName(TEXT("ConfigEditorPropertyHelper_"));
	ConfigHelperName += SelectedTargetPlatform;

	PropHelper = FindObject<UConfigHierarchyPropertyView>(GetTransientPackage(), *ConfigHelperName);
	if (!PropHelper.IsValid())
	{
		PropHelper = NewObject<UConfigHierarchyPropertyView>(GetTransientPackage(), *ConfigHelperName);
		PropHelper->AddToRoot();
	}

	PropHelper->EditProperty = EditProperty.Get();
	FString ClassConfigName = PropHelper->EditProperty->GetOwnerClass()->ClassConfigName.ToString();

	if (FConfigCacheIni* PlatformConfigCache = FConfigCacheIni::ForPlatform(*SelectedTargetPlatform))
	{
		if (FConfigBranch* Branch = PlatformConfigCache->FindBranch(*ClassConfigName, ClassConfigName))
		{
			for (const auto& IniFile : Branch->Hierarchy)
			{
				UPropertyConfigFileDisplayRow* ConfigFilePropertyObj = NewObject<UPropertyConfigFileDisplayRow>(GetTransientPackage(), *IniFile.Value);
				ConfigFilePropertyObj->InitWithConfigAndProperty(FString(IniFile.Value), PropHelper->EditProperty.Get());

				PropHelper->ConfigFilePropertyObjects.Add(ConfigFilePropertyObj);
			}
		}
	}

	DetailsView->SetObject(PropHelper.Get());
}

void SConfigEditor::HandleTargetPlatformChanged()
{
	CreateDisplayObjectForSelectedTargetPlatform();
}


#undef LOCTEXT_NAMESPACE
