// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/WeakFieldPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FProperty;
class SWidget;

#define LOCTEXT_NAMESPACE "ConfigEditor"

class FConfigCacheIni;
class IDetailsView;
class UConfigHierarchyPropertyView;

/*-----------------------------------------------------------------------------
   SConfigEditor
-----------------------------------------------------------------------------*/
class SConfigEditor : public SCompoundWidget
{
public:
	SLATE_USER_ARGS(SConfigEditor) {}
	SLATE_END_ARGS()

	//~ Begin SCompoundWidget|SWidget Interface
	virtual void Construct(const FArguments& InArgs, TWeakFieldPtr<FProperty> InEditProperty);
	//~ End SCompoundWidget|SWidget Interface

private:
	/**
	 * Handle a change of target platform in the Config Editor UI
	 */
	void HandleTargetPlatformChanged();
	
	// Panel used to select an available target platform.
	TSharedPtr<class STargetPlatformSelector> TargetPlatformSelection;
	
	/**
	 * Create the displayable area object for the selected platform
	 */
	void CreateDisplayObjectForSelectedTargetPlatform();

private:
	// This is an object we use to create a config hierarchy display for a property
	TWeakObjectPtr<UConfigHierarchyPropertyView> PropHelper;

	// The display area for the config hierarchy editor.
	TSharedPtr<SWidget> PropertyValueEditor;

	// The bulk of the display of this hierarchy.
	TSharedPtr<IDetailsView> DetailsView;

	// Keep track of the property we are viewing.
	TWeakFieldPtr<FProperty> EditProperty;
};

#undef LOCTEXT_NAMESPACE
