// Copyright Epic Games, Inc. All Rights Reserved.

#include "STargetPlatformSelector.h"

#include "Containers/UnrealString.h"
#include "HAL/PlatformCrt.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Internationalization/Internationalization.h"
#include "Layout/Children.h"
#include "Layout/Margin.h"
#include "Misc/Attribute.h"
#include "Misc/CoreMisc.h"
#include "SlateOptMacros.h"
#include "SlotBase.h"
#include "Styling/CoreStyle.h"
#include "Styling/ISlateStyle.h"
#include "Types/SlateStructs.h"
#include "Widgets/Input/STextComboBox.h"
#include "Widgets/Layout/SBorder.h"
#include "Widgets/Layout/SBox.h"
#include "Widgets/SBoxPanel.h"
#include "Widgets/Text/STextBlock.h"

#define LOCTEXT_NAMESPACE "ConfigEditor"


BEGIN_SLATE_FUNCTION_BUILD_OPTIMIZATION
void STargetPlatformSelector::Construct(const FArguments& InArgs)
{
	OnTargetPlatformChanged = InArgs._OnTargetPlatformChanged;

	CollateAvailableTargetPlatformEntries();

	this->ChildSlot
	[
		SNew(SBorder)
		.BorderImage(FCoreStyle::Get().GetBrush("ToolPanel.GroupBorder"))
		.Padding(FMargin(8.0f, 8.0f, 16.0f, 8.0f))
		[
			SNew(SHorizontalBox)
			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Right)
			.VAlign(EVerticalAlignment::VAlign_Center)
			.Padding(FMargin(8.0f, 8.0f, 8.0f, 8.0f))
			[
				SNew(STextBlock)
				.Text(LOCTEXT("TargetPlatformSelectionLabel", "Target platform:"))
			]
			+ SHorizontalBox::Slot()
			.HAlign(EHorizontalAlignment::HAlign_Right)
			.Padding(FMargin(0.0f, 4.0f))
			.AutoWidth()
			[
				SNew(SBox)
				.WidthOverride(150.0f)
				[
					SAssignNew(AvailableTargetPlatformComboBox, STextComboBox)
					.OptionsSource(&TargetPlatformOptionsSource)
					.InitiallySelectedItem(SelectedTargetPlatform)
					.OnSelectionChanged(this, &STargetPlatformSelector::HandleTargetPlatformChanged)
				]
			]
		]
	];

}
END_SLATE_FUNCTION_BUILD_OPTIMIZATION


TSharedPtr<FString> STargetPlatformSelector::GetSelectedTargetPlatform()
{
	return SelectedTargetPlatform;
}


void STargetPlatformSelector::HandleTargetPlatformChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo)
{
	SelectedTargetPlatform = NewValue;
	OnTargetPlatformChanged.ExecuteIfBound();
}


void STargetPlatformSelector::CollateAvailableTargetPlatformEntries()
{
	ITargetPlatformManagerModule& TPM = GetTargetPlatformManagerRef();
	TArray<ITargetPlatform*> ActiveTargetPlatforms = TPM.GetTargetPlatforms();
	for (ITargetPlatform* TargetPlatform : ActiveTargetPlatforms)
	{
		const FString CurrentIniName = TargetPlatform->IniPlatformName();
		if (!CurrentIniName.IsEmpty())
		{
			const TSharedPtr<FString>* ExistingEntry = TargetPlatformOptionsSource.FindByPredicate([CurrentIniName](const TSharedPtr<FString>& CurElement)
			{
				return *CurElement == CurrentIniName;
			});

			if (ExistingEntry == nullptr || !(*ExistingEntry).IsValid())
			{
				TargetPlatformOptionsSource.Add(MakeShareable(new FString(TargetPlatform->IniPlatformName())));
			}
		}
	}

	SelectedTargetPlatform = TargetPlatformOptionsSource[0];
}


#undef LOCTEXT_NAMESPACE
