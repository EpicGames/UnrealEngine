// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Delegates/Delegate.h"
#include "Templates/SharedPointer.h"
#include "Types/SlateEnums.h"
#include "Widgets/DeclarativeSyntaxSupport.h"
#include "Widgets/SCompoundWidget.h"

class FString;
class STextComboBox;

#define LOCTEXT_NAMESPACE "ConfigEditor"

/*-----------------------------------------------------------------------------
   STargetPlatformSelector
-----------------------------------------------------------------------------*/
class STargetPlatformSelector : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(STargetPlatformSelector){}
		SLATE_EVENT(FSimpleDelegate, OnTargetPlatformChanged)
	SLATE_END_ARGS()

	//~ Begin SCompoundWidget|SWidget Interface
	void Construct(const FArguments& InArgs);
	//~ End SCompoundWidget|SWidget Interface

	/**
	 * Access to the selected target platform in this widget.
	 *
	 * @return - The selected platform from the target platform ComboBox.
	 */
	TSharedPtr<FString> GetSelectedTargetPlatform();

private:
	 
	/**
	 * Callback when the user has made a change to the target platform.
	 */
	void HandleTargetPlatformChanged(TSharedPtr<FString> NewValue, ESelectInfo::Type SelectInfo);

	/**
	 * Build a list of available target platforms.
	 */
	void CollateAvailableTargetPlatformEntries();

private:

	// The currently selected target platform in this widget.
	TSharedPtr<FString> SelectedTargetPlatform;

	// The combo widget used to make a change to selected target platform.
	TSharedPtr<STextComboBox> AvailableTargetPlatformComboBox;

	// The list of available target platforms for the combo widget.
	TArray< TSharedPtr<FString> > TargetPlatformOptionsSource;

	// Delegate called to let the listener know there has been a change to the target platform.
	FSimpleDelegate OnTargetPlatformChanged;
};

#undef LOCTEXT_NAMESPACE
