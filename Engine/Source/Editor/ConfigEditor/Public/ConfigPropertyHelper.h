// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
 
#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "UObject/FieldPath.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakFieldPtr.h"
#include "UObject/WeakObjectPtr.h"

#include "ConfigPropertyHelper.generated.h"

class FProperty;
template <typename T> struct TObjectPtr;

UENUM()
enum EConfigFileSourceControlStatus : int
{
	CFSCS_Unknown UMETA(DisplayName=Unknown),
	CFSCS_Writable UMETA(DisplayName = "Available to edit"),
	CFSCS_Locked UMETA(DisplayName = "File is locked"),
};

UCLASS(MinimalAPI)
class UPropertyConfigFileDisplayRow : public UObject
{
	GENERATED_BODY()
public:

	void InitWithConfigAndProperty(const FString& InConfigFileName, FProperty* InEditProperty);

public:
	UPROPERTY(Transient, Category = Helper, VisibleAnywhere)
	FString ConfigFileName;

	FString NormalizedConfigFileName;
	TObjectPtr<UObject> TempConfigObject;

	UPROPERTY(Transient, Category = Helper, EditAnywhere, meta=(EditCondition="bIsFileWritable"))
	TFieldPath<FProperty> ExternalProperty;

	UPROPERTY(Transient, Category = Helper, VisibleAnywhere)
	bool bIsFileWritable;
};


UCLASS(MinimalAPI)
class UConfigHierarchyPropertyView : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY(Transient, Category=Helper, EditAnywhere)
	TFieldPath<FProperty> EditProperty;

	UPROPERTY(Transient, Category = Helper, EditAnywhere)
	TArray<TObjectPtr<UPropertyConfigFileDisplayRow>> ConfigFilePropertyObjects;
};
