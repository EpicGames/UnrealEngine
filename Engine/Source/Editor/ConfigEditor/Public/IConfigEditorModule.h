// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Widgets/SWidget.h"
#include "Modules/ModuleInterface.h"

/*-----------------------------------------------------------------------------
   IConfigEditorModule
-----------------------------------------------------------------------------*/
class IConfigEditorModule : public IModuleInterface
{
public:
	/**
	 * Creates a config hierarchy editor for the given property 
	 */
	virtual void CreateHierarchyEditor(FProperty* EditProperty) = 0;

	/**
	 * Maintain a reference to a value widget used in the config editor for the current property.
	 */
	virtual void AddExternalPropertyValueWidgetAndConfigPairing(const FString& ConfigFile, const TSharedPtr<SWidget> ValueWidget) = 0;

	/**
	 * Access a reference to a value widget used in the config editor for the current property.
	 *
	 * @return -	A widget which represents the config value for a specific property. This is a different widget based on the property type
	 *				ie. a combobox for enum, a check box for bool...
	 */
	virtual TSharedRef<SWidget> GetValueWidgetForConfigProperty(const FString& ConfigFile) = 0;
};
