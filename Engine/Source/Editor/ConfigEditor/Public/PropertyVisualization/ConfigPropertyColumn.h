// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTableCustomColumn.h"
#include "IPropertyTableUtilities.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API CONFIGEDITOR_API

class FProperty;
class IPropertyTableCell;
class IPropertyTableCellPresenter;
class IPropertyTableColumn;
class IPropertyTableUtilities;
class SWidget;

/**
* A property table custom column used to display source control condition of files.
*/
class FConfigPropertyCustomColumn : public IPropertyTableCustomColumn
{
public:
	FConfigPropertyCustomColumn(){}

	/** Begin IPropertyTableCustomColumn interface */
	UE_API virtual bool Supports(const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities) const override;
	UE_API virtual TSharedPtr< SWidget > CreateColumnLabel(const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style) const override;
	UE_API virtual TSharedPtr< IPropertyTableCellPresenter > CreateCellPresenter(const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style) const override;
	/** End IPropertyTableCustomColumn interface */

	/* The property type which can be displayed in this column */
	FProperty* EditProperty;
};

#undef UE_API
