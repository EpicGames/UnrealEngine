// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "IPropertyTableCellPresenter.h"
#include "IPropertyTableColumn.h"
#include "IPropertyTableCustomColumn.h"
#include "IPropertyTableUtilities.h"
#include "Internationalization/Text.h"
#include "Templates/SharedPointer.h"
#include "UObject/NameTypes.h"

#define UE_API CONFIGEDITOR_API

class FProperty;
class IPropertyHandle;
class IPropertyTableCell;
class IPropertyTableColumn;
class IPropertyTableUtilities;
class SWidget;

class FConfigPropertyConfigFileStateCellPresenter : public TSharedFromThis< FConfigPropertyConfigFileStateCellPresenter >, public IPropertyTableCellPresenter
{
public:

	FConfigPropertyConfigFileStateCellPresenter(const TSharedPtr< IPropertyHandle > PropertyHandle);
	virtual ~FConfigPropertyConfigFileStateCellPresenter() = default;


	/** Begin IPropertyTableCellPresenter interface */
	virtual TSharedRef< class SWidget > ConstructDisplayWidget() override;
	virtual bool RequiresDropDown() override;
	virtual TSharedRef< class SWidget > ConstructEditModeCellWidget() override;
	virtual TSharedRef< class SWidget > ConstructEditModeDropDownWidget() override;
	virtual TSharedRef< class SWidget > WidgetToFocusOnEdit() override;
	virtual bool HasReadOnlyEditMode() override;
	virtual FString GetValueAsString() override;

	virtual FText GetValueAsText() override;
	/** End IPropertyTableCellPresenter interface */

protected:

	/**
	* The text we display - we just store this as we never need to *edit* properties in the stats viewer,
	* only view them. Therefore we dont need to dynamically update this string.
	*/
	FText Text;
};



/**
* A property table custom column used to display source control condition of files.
*/
class FConfigPropertyConfigFileStateCustomColumn : public IPropertyTableCustomColumn
{
public:
	FConfigPropertyConfigFileStateCustomColumn(){}

	/** Begin IPropertyTableCustomColumn interface */
	UE_API virtual bool Supports(const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities) const override;
	UE_API virtual TSharedPtr< SWidget > CreateColumnLabel(const TSharedRef< IPropertyTableColumn >& Column, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style) const override;
	UE_API virtual TSharedPtr< IPropertyTableCellPresenter > CreateCellPresenter(const TSharedRef< IPropertyTableCell >& Cell, const TSharedRef< IPropertyTableUtilities >& Utilities, const FName& Style) const override;
	/** End IPropertyTableCustomColumn interface */

	/* The property type which can be displayed in this column */
	FProperty* SupportedProperty;
};

#undef UE_API
